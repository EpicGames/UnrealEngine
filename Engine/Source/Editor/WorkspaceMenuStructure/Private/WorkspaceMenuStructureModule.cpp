// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkspaceMenuStructureModule.h"

#include "Framework/Docking/WorkspaceItem.h"
#if WITH_EDITOR
#include "Interfaces/IEditorStyleModule.h"
#endif
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Textures/SlateIcon.h"
#include "WorkspaceMenuStructure.h"


IMPLEMENT_MODULE( FWorkspaceMenuStructureModule, WorkspaceMenuStructure );

#define LOCTEXT_NAMESPACE "UnrealEditor"

class FWorkspaceMenuStructure : public IWorkspaceMenuStructure
{
public:
	virtual TSharedRef<FWorkspaceItem> GetStructureRoot() const override
	{
		return MenuRoot.ToSharedRef();
	}
	virtual TSharedRef<FWorkspaceItem> GetToolsStructureRoot() const override
	{
		return ToolsMenuRoot.ToSharedRef();
	}

	virtual TSharedRef<FWorkspaceItem> GetLevelEditorCategory() const override
	{
		return LevelEditorCategory.ToSharedRef();
	}

	virtual TSharedRef<FWorkspaceItem> GetLevelEditorViewportsCategory() const override
	{
		return LevelEditorViewportsCategory.ToSharedRef();
	}

	virtual TSharedRef<FWorkspaceItem> GetLevelEditorDetailsCategory() const override
	{
		return LevelEditorDetailsCategory.ToSharedRef();
	}

	virtual TSharedRef<FWorkspaceItem> GetLevelEditorModesCategory() const override
	{
		return LevelEditorModesCategory.ToSharedRef();
	}

	virtual TSharedRef<FWorkspaceItem> GetLevelEditorCinematicsCategory() const override
	{
		return LevelEditorCinematicsCategory.ToSharedRef();
	}

	virtual TSharedRef<FWorkspaceItem> GetLevelEditorMediaCategory() const override
	{
		return LevelEditorMediaCategory.ToSharedRef();
	}

	virtual TSharedRef<FWorkspaceItem> GetLevelEditorVirtualProductionCategory() const override
	{
		return LevelEditorVirtualProductionCategory.ToSharedRef();
	}

	virtual TSharedRef<FWorkspaceItem> GetLevelEditorWorldPartitionCategory() const override
	{
		return LevelEditorWorldPartitionCategory.ToSharedRef();
	}
		
	virtual TSharedRef<FWorkspaceItem> GetLevelEditorOutlinerCategory() const override
	{
		return LevelEditorOutlinerCategory.ToSharedRef();
	}

	virtual TSharedRef<FWorkspaceItem> GetToolsCategory() const override
	{
		return ToolsCategory.ToSharedRef();
	}

	virtual TSharedRef<FWorkspaceItem> GetDeveloperToolsDebugCategory() const override
	{
		return DeveloperToolsDebugCategory.ToSharedRef();
	}

	virtual TSharedRef<FWorkspaceItem> GetDeveloperToolsLogCategory() const override
	{
		return DeveloperToolsLogCategory.ToSharedRef();
	}

	virtual TSharedRef<FWorkspaceItem> GetDeveloperToolsProfilingCategory() const override
	{
		return DeveloperToolsProfilingCategory.ToSharedRef();
	}

	virtual TSharedRef<FWorkspaceItem> GetDeveloperToolsAuditCategory() const override
	{
		return DeveloperToolsAuditCategory.ToSharedRef();
	}	

	virtual TSharedRef<FWorkspaceItem> GetDeveloperToolsPlatformsCategory() const override
	{
		return DeveloperToolsPlatformsCategory.ToSharedRef();
	}

	virtual TSharedRef<FWorkspaceItem> GetDeveloperToolsMiscCategory() const override
	{
		return DeveloperToolsMiscCategory.ToSharedRef();
	}

	virtual TSharedRef<FWorkspaceItem> GetAutomationToolsCategory() const override
	{
		return AutomationToolsCategory.ToSharedRef();
	}

	virtual TSharedRef<FWorkspaceItem> GetEditOptions() const override
	{
		return EditOptions.ToSharedRef();
	}

	void ResetLevelEditorCategory()
	{
		LevelEditorCategory->ClearItems();
		LevelEditorViewportsCategory = LevelEditorCategory->AddGroup("Viewports",
			LOCTEXT("WorkspaceMenu_LevelEditorViewportCategory", "Viewports"),
			LOCTEXT("WorkspaceMenu_LevelEditorViewportCategoryTooltip", "Open a Viewport tab."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.Tabs.Viewports"), true);
		LevelEditorDetailsCategory = LevelEditorCategory->AddGroup("Details",
			LOCTEXT("WorkspaceMenu_LevelEditorDetailCategory", "Details"),
			LOCTEXT("WorkspaceMenu_LevelEditorDetailCategoryTooltip", "Open a Details tab."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.Tabs.Details"), true);
		LevelEditorCinematicsCategory = LevelEditorCategory->AddGroup("Cinematics",
			LOCTEXT("WorkspaceMenu_LevelEditorCinematicsCategory", "Cinematics"),
			LOCTEXT("WorkspaceMenu_LevelEditorCinematicsCategoryTooltip", "Open a Cinematics tab."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.Tabs.Cinematics"), true);
		LevelEditorMediaCategory = LevelEditorCategory->AddGroup("Media",
			LOCTEXT("WorkspaceMenu_LevelEditorMediaCategory", "Media"),
			LOCTEXT("WorkspaceMenu_LevelEditorMediaCategoryTooltip", "Open a Media tab."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.Tabs.Media"), true);
		LevelEditorVirtualProductionCategory = LevelEditorCategory->AddGroup("VirtualProduction",
			LOCTEXT("WorkspaceMenu_LevelEditorVirtualProductionCategory", "Virtual Production"),
			LOCTEXT("WorkspaceMenu_LevelEditorVirtualProductionCategoryTooltip", "Open a Virtual Production tab."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.Tabs.VirtualProduction"), true);
		LevelEditorModesCategory = LevelEditorCategory->AddGroup("EditorModes",
			LOCTEXT("WorkspaceMenu_LevelEditorToolsCategory", "Editor Modes"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.Tabs.EditorModes"), true);
		LevelEditorWorldPartitionCategory = LevelEditorCategory->AddGroup("WorldPartition",
			LOCTEXT("WorkspaceMenu_LevelEditorWorldPartitionCategory", "World Partition"),
			LOCTEXT("WorkspaceMenu_LevelEditorWorldPartitionCategoryTooltip", "Open a World Partition tab."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.Tabs.WorldPartition"), true);
		LevelEditorOutlinerCategory = LevelEditorCategory->AddGroup("Outliner",
			LOCTEXT("WorkspaceMenu_LevelEditorOutlinerCategory", "Outliner"),
			LOCTEXT("WorkspaceMenu_LevelEditorOutlinerCategoryTooltip", "Open an Outliner tab."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"), true);
	}

	void ResetToolsCategory()
	{
		// Developer tools sections
		DeveloperToolsCategory->ClearItems();
		DeveloperToolsSubMenuDebugCategory = DeveloperToolsCategory->AddGroup("Debug",
			LOCTEXT("WorkspaceMenu_DeveloperToolsDebugCategory", "Debug"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.Tabs.Debug"), true);

		DeveloperToolsDebugCategory = DeveloperToolsSubMenuDebugCategory->AddGroup(
			"Debug", LOCTEXT("WorkspaceMenu_DeveloperToolsDebugCategory", "Debug"), FSlateIcon(), true);
		AutomationToolsCategory = DeveloperToolsSubMenuDebugCategory->AddGroup(
			"Testing", LOCTEXT("WorkspaceMenu_AutomationToolsCategory", "Testing"), FSlateIcon(), true);

		DeveloperToolsProfilingCategory = DeveloperToolsCategory->AddGroup("Profile",
			LOCTEXT("WorkspaceMenu_DeveloperToolsProfilingCategory", "Profile"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.Profile"), true);
		DeveloperToolsAuditCategory = DeveloperToolsCategory->AddGroup("Audit",
			LOCTEXT("WorkspaceMenu_DeveloperToolsAuditCategory", "Audit"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.Audit"), true);
		DeveloperToolsPlatformsCategory = DeveloperToolsCategory->AddGroup("Platforms",
			LOCTEXT("WorkspaceMenu_DeveloperToolsPlatforms:WCategory", "Platforms"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.Platforms"), true);
		DeveloperToolsMiscCategory = DeveloperToolsCategory->AddGroup("Miscellaneous",
			LOCTEXT("WorkspaceMenu_DeveloperToolsMiscCategory", "Miscellaneous"), FSlateIcon(), true);
	}

public:
	FWorkspaceMenuStructure()
		: MenuRoot(FWorkspaceItem::NewGroup(LOCTEXT("WorkspaceMenu_Root", "Menu Root")))
		, ToolsMenuRoot(FWorkspaceItem::NewGroup(LOCTEXT("WorkspaceToolsMenu_Root", "Tools Menu Root")))
		, LevelEditorCategory(MenuRoot->AddGroup(
			  "LevelEditor", LOCTEXT("WorkspaceMenu_LevelEditorCategory", "Level Editor"), FSlateIcon(), true))
		, ToolsCategory(
			  ToolsMenuRoot->AddGroup("Tools", LOCTEXT("WorkspaceMenu_ToolsCategory", "Tools"), FSlateIcon(), true))
		, DeveloperToolsCategory(ToolsMenuRoot->AddGroup(
			  "Instrumentation", LOCTEXT("WorkspaceMenu_DeveloperToolsCategory", "Instrumentation"), FSlateIcon()))
		, DeveloperToolsLogCategory(
			  MenuRoot->AddGroup("Log", LOCTEXT("WorkspaceMenu_DeveloperToolsLogCategory", "Log"), FSlateIcon(), true))
		, EditOptions(FWorkspaceItem::NewGroup(LOCTEXT("WorkspaceEdit_Options", "Edit Options")))
	{
		ResetLevelEditorCategory();
		ResetToolsCategory();
	}

	virtual ~FWorkspaceMenuStructure() {}

private:
	TSharedPtr<FWorkspaceItem> MenuRoot;
	TSharedPtr<FWorkspaceItem> ToolsMenuRoot;
	
	TSharedPtr<FWorkspaceItem> LevelEditorCategory;
	TSharedPtr<FWorkspaceItem> LevelEditorViewportsCategory;
	TSharedPtr<FWorkspaceItem> LevelEditorDetailsCategory;
	TSharedPtr<FWorkspaceItem> LevelEditorCinematicsCategory;
	TSharedPtr<FWorkspaceItem> LevelEditorMediaCategory;
	TSharedPtr<FWorkspaceItem> LevelEditorVirtualProductionCategory;
	TSharedPtr<FWorkspaceItem> LevelEditorModesCategory;
	TSharedPtr<FWorkspaceItem> LevelEditorWorldPartitionCategory;
	TSharedPtr<FWorkspaceItem> LevelEditorOutlinerCategory;

	TSharedPtr<FWorkspaceItem> ToolsCategory;
	TSharedPtr<FWorkspaceItem> DeveloperToolsCategory;
	TSharedPtr<FWorkspaceItem> DeveloperToolsLogCategory;
	TSharedPtr<FWorkspaceItem> DeveloperToolsSubMenuDebugCategory;
	TSharedPtr<FWorkspaceItem> DeveloperToolsDebugCategory;
	TSharedPtr<FWorkspaceItem> DeveloperToolsAuditCategory;
	TSharedPtr<FWorkspaceItem> DeveloperToolsProfilingCategory;
	TSharedPtr<FWorkspaceItem> DeveloperToolsPlatformsCategory;
	TSharedPtr<FWorkspaceItem> DeveloperToolsMiscCategory;
	
	TSharedPtr<FWorkspaceItem> AutomationToolsCategory;
	
	TSharedPtr<FWorkspaceItem> EditOptions;
};

void FWorkspaceMenuStructureModule::StartupModule()
{
#if WITH_EDITOR
	// The workspace menu categories rely on icons from the editor style.
	FModuleManager::LoadModuleChecked<IEditorStyleModule>("EditorStyle");
#endif

	WorkspaceMenuStructure = MakeShareable(new FWorkspaceMenuStructure);
}

void FWorkspaceMenuStructureModule::ShutdownModule()
{
	WorkspaceMenuStructure.Reset();
}

const IWorkspaceMenuStructure& FWorkspaceMenuStructureModule::GetWorkspaceMenuStructure() const
{
	check(WorkspaceMenuStructure.IsValid());
	return *WorkspaceMenuStructure;
}

void FWorkspaceMenuStructureModule::ResetLevelEditorCategory()
{
	check(WorkspaceMenuStructure.IsValid());
	WorkspaceMenuStructure->ResetLevelEditorCategory();
}

void FWorkspaceMenuStructureModule::ResetToolsCategory()
{
	check(WorkspaceMenuStructure.IsValid());
	WorkspaceMenuStructure->ResetToolsCategory();
}

#undef LOCTEXT_NAMESPACE
