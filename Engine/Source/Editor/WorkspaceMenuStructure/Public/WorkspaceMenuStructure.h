// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Docking/WorkspaceItem.h"

/**
 * Describes the workspace menu from which tabs are summoned
 *
 *  +-Asset Editor Tabs
 *    |-Tab
 *    |-...
 *    |
 *  +-Level Editor Tabs
 *    |-Tab
 *    |-...
 *    | \ Viewports >
 *  +-Tools Tabs
 *    |-Tab
 *    |-...
 *    \ Automation >
 *    \ Developer Tools >
 */

class FWorkspaceItem;

class IWorkspaceMenuStructure
{
public:
	/** Get the root of the menu structure. Pass this into PopulateTabSpawnerMenu() */
	virtual TSharedRef<FWorkspaceItem> GetStructureRoot() const = 0;

	/** Alternate root for the Tools menu */
	virtual TSharedRef<FWorkspaceItem> GetToolsStructureRoot() const = 0;

	/** See diagram above */
	virtual TSharedRef<FWorkspaceItem> GetLevelEditorCategory() const = 0;

	/** See diagram above */
	virtual TSharedRef<FWorkspaceItem> GetLevelEditorViewportsCategory() const = 0;

	/** See diagram above */
	virtual TSharedRef<FWorkspaceItem> GetLevelEditorDetailsCategory() const = 0;

	/** See diagram above */
	virtual TSharedRef<FWorkspaceItem> GetLevelEditorCinematicsCategory() const = 0;

	/** See diagram above */
	virtual TSharedRef<FWorkspaceItem> GetLevelEditorMediaCategory() const = 0;

	/** See diagram above */
	virtual TSharedRef<FWorkspaceItem> GetLevelEditorVirtualProductionCategory() const = 0;

	/** See diagram above */
	virtual TSharedRef<FWorkspaceItem> GetLevelEditorWorldPartitionCategory() const = 0;
	
	/** See diagram above */
	virtual TSharedRef<FWorkspaceItem> GetLevelEditorOutlinerCategory() const = 0;

	/** See diagram above */
	virtual TSharedRef<FWorkspaceItem> GetLevelEditorModesCategory() const = 0;

	/** See diagram above */
	virtual TSharedRef<FWorkspaceItem> GetToolsCategory() const = 0;

	/** See diagram above */
	virtual TSharedRef<FWorkspaceItem> GetDeveloperToolsDebugCategory() const = 0;

	/** See diagram above */
	virtual TSharedRef<FWorkspaceItem> GetDeveloperToolsLogCategory() const = 0;

	/** See diagram above */
	virtual TSharedRef<FWorkspaceItem> GetDeveloperToolsProfilingCategory() const = 0;	
	
	/** See diagram above */
	virtual TSharedRef<FWorkspaceItem> GetDeveloperToolsAuditCategory() const = 0;	

	/** See diagram above */
	virtual TSharedRef<FWorkspaceItem> GetDeveloperToolsPlatformsCategory() const = 0;	
	
	/** See diagram above */
	virtual TSharedRef<FWorkspaceItem> GetDeveloperToolsMiscCategory() const = 0;

	/** See diagram above */
	virtual TSharedRef<FWorkspaceItem> GetAutomationToolsCategory() const = 0;

	/** Get the root of the edit menu structure. Pass this into PopulateTabSpawnerMenu() */
	virtual TSharedRef<FWorkspaceItem> GetEditOptions() const = 0;
};
