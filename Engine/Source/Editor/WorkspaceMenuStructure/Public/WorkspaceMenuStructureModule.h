// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

class FWorkspaceMenuStructure;
class IWorkspaceMenuStructure;

/**
 * The module that defines a structure of the workspace menu.
 * Tab spawners place themselves into one of the categories/groups
 * in this structure upon registration.
 */
class FWorkspaceMenuStructureModule : public IModuleInterface
{

public:

	virtual ~FWorkspaceMenuStructureModule() = default;

	virtual void StartupModule();

	virtual void ShutdownModule();

	/** @return The menu structure that is populated by tab spawners */
	virtual const IWorkspaceMenuStructure& GetWorkspaceMenuStructure() const;

	/** Reset the "Level Editor" category to its default state */
	virtual void ResetLevelEditorCategory();

	/** Reset the "Tools" category to its default state */
	virtual void ResetToolsCategory();
	
private:
	TSharedPtr<class FWorkspaceMenuStructure> WorkspaceMenuStructure;
};


namespace WorkspaceMenu
{
	inline FWorkspaceMenuStructureModule& GetModule()
	{
		return FModuleManager::LoadModuleChecked<FWorkspaceMenuStructureModule>("WorkspaceMenuStructure");
	}

	inline const IWorkspaceMenuStructure& GetMenuStructure()
	{
		return GetModule().GetWorkspaceMenuStructure();
	}
}
