// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Game, TargetType.Program, TargetType.Server)]
public class WorkspaceMenuStructure : ModuleRules
{
	public WorkspaceMenuStructure(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"SlateCore",
				"Slate"
			}
		);
		
		if (Target.bCompileAgainstEditor)
		{
			PublicDependencyModuleNames.Add("EditorStyle");
		}
	}
}
